[Plugins]
+ProgramEnabledPlugins=AudioInsights
+ProgramEnabledPlugins=AudioModulationInsights
+ProgramEnabledPlugins=GameplayInsights
+ProgramEnabledPlugins=SlateInsights
+ProgramEnabledPlugins=RenderGraphInsights
+ProgramEnabledPlugins=IoStoreInsights
+ProgramEnabledPlugins=UdpMessaging
+ProgramEnabledPlugins=TcpMessaging
+ProgramEnabledPlugins=NetworkPredictionInsights
+ProgramEnabledPlugins=VisualStudioSourceCodeAccess
+ProgramEnabledPlugins=XcodeSourceCodeAccess
+ProgramEnabledPlugins=MassInsights
+ProgramEnabledPlugins=ChaosInsights
+ProgramEnabledPlugins=MessagingInsights
+ProgramEnabledPlugins=HttpInsights

[/Script/MacTargetPlatform.XcodeProjectSettings]
TemplateMacPlist=(FilePath="/Engine/Source/Programs/UnrealInsights/Resources/Mac/Info.plist")
