<MemoryBudget>

	<!--
		1 KB = 1000 Bytes
		1 MB = 10^6 Bytes = 1,000,000 Bytes
		1 GB = 10^9 Bytes = 1,000,000,000 Bytes
		1 KiB = 2^10 Bytes = 1024 Bytes
		1 MiB = 2^20 Bytes = 1,048,576 Bytes
		1 GiB = 2^30 Bytes = 1,073,741,824 Bytes
	-->

	<!--
		<budget name=""> ...sets...platforms... </budget>
		where:
			name = the budget mode name
			sets = <set/> nodes
			platforms = <platform/> nodes

		<set name=""> ...trackers... </set>
		where:
			name = "System", "Asset" or "AssetClass"
			trackers = <tracker/> nodes
			
		<tracker name=""> ...tags... </tracker>
		where:
			name = "Default" or "Platform"
			tags = <tag/> nodes
			groups = <group/> nodes

		<grouping> ...groups... </grouping>
		where:
			groups = <group/> nodes

		<tag name="" mem-max="0"/>
		where:
			name = the LLM tag name
			mem-max = the memory budget value (high threshold value); in [bytes], if no unit is specified

		<group name="" include="" exclude="" min-max=""/>
		where:
			name = the group name
			include = the regex for tag names to be included
			exclude = the regex for tag names to be excluded
			mem-max = the memory budget value (high threshold value); in [bytes], if no unit is specified

		<platform name=""> ...sets... </platform>
		where:
			name = the platform name
			sets = <set/> nodes
	-->

	<budget name="Default">

		<!-- System tag set -->
		<set name="System">
			<tracker name="Default">
				<tag name="Animation"                       mem-max="90 MiB"/>
				<tag name="AssetManager"                    mem-max="30 MiB"/>
				<tag name="AssetRegistry"                   mem-max="200 MiB"/>
				<tag name="AsyncLoading"                    mem-max="225 MiB"/>
				<tag name="Audio"                           mem-max="180 MiB"/>
				<tag name="EngineInit"                      mem-max="16 MiB"/>
				<tag name="EnginePreInit"                   mem-max="100 MiB"/>
				<tag name="EngineMisc"                      mem-max="50 MiB"/>
				<tag name="FileSystem"                      mem-max="100 MiB"/>
				<tag name="FName"                           mem-max="6 MiB"/>
				<tag name="GC"                              mem-max="20 MiB"/>
				<tag name="Localization"                    mem-max="50 MiB"/>
				<tag name="Materials"                       mem-max="30 MiB"/>
				<tag name="Meshes"                          mem-max="150 MiB"/>
				<tag name="Meshes/StaticMesh"               mem-max="20 MiB"/>
				<tag name="Meshes/StaticMesh/Serialize"     mem-max="10 MiB"/>
				<tag name="Meshes/SkeletalMesh"             mem-max="50 MiB"/>
				<tag name="Meshes/SkeletalMesh/Serialize"   mem-max="50 MiB"/>
				<tag name="Niagara"                         mem-max="100 MiB"/>
				<tag name="Particles"                       mem-max="100 MiB"/>
				<tag name="Physics"                         mem-max="500 MiB"/>
				<tag name="RenderTargets"                   mem-max="750 MiB"/>
				<tag name="RHIMisc"                         mem-max="50 MiB"/>
				<tag name="SceneRender"                     mem-max="50 MiB"/>
				<tag name="Shaders"                         mem-max="100 MiB"/>
				<tag name="Stats"                           mem-max="5 MiB"/>
				<tag name="TextureMetaData"                 mem-max="5 MiB"/>
				<tag name="Textures"                        mem-max="2 GiB"/>
				<tag name="UI"                              mem-max="100 MiB"/>
				<tag name="UObject"                         mem-max="250 MiB"/>
				<tag name="ThreadStack"                     mem-max="300 MiB"/>
				<tag name="Untagged"                        mem-max="1 MiB"/>
				<tag name="Total"                           mem-max="2.11 GiB"/>
				<tag name="TrackedTotal"                    mem-max="2.1 GiB"/>
				<tag name="Untracked"                       mem-max="10 MiB"/>
			</tracker>
			<tracker name="Platform">
				<tag name="ThreadStack"                     mem-max="300 MiB"/>
				<tag name="Untagged"                        mem-max="2 MiB"/>
				<tag name="Graphics"                        mem-max="1 GiB"/>
				<tag name="FMalloc"                         mem-max="2.5 GiB"/>
				<tag name="LLMOverhead"                     mem-max="500 MiB"/>
				<tag name="Total"                           mem-max="2.21 GiB"/>
				<tag name="TrackedTotal"                    mem-max="2.2 GiB"/>
				<tag name="Untracked"                       mem-max="20 MiB"/>
			</tracker>
		</set>

		<!-- Asset tag set -->
		<set name="Asset">
			<tracker name="Default">
				<tag name="UntaggedAsset" mem-max="2 GiB"/>
			</tracker>
			<tracker name="Platform">
				<tag name="UntaggedAsset" mem-max="3 GiB"/>
			</tracker>
			<grouping>
				<!-- Asset groups -->
				<group name="Engine"  include="^/Engine/.*"  mem-max="1 GiB"/>
				<group name="Engine2" include="^\.\./\.\./\.\./Engine/.*"  mem-max="1 GiB"/> <!-- assets stating with "../../../Engine/" -->
				<group name="Game"    include="^/Game/.*"    mem-max="2 GiB"/>
				<group name="Script"  include="^/Script/.*"  mem-max="120 MiB"/>
				<!-- A group for all other assets -->
				<group name="Other" include=".*" exclude="UntaggedAsset" mem-max="100 MiB"/>
			</grouping>
		</set>

		<!-- AssetClass tag set -->
		<set name="AssetClass">
			<tracker name="Default">
				<tag name="UntaggedAssetClass" mem-max="2 GiB"/>
			</tracker>
			<tracker name="Platform">
				<tag name="UntaggedAssetClass" mem-max="1 GiB"/>
			</tracker>
			<grouping>
				<!-- AssetClass groups -->
				<group name="Engine"  include="^/Engine/.*"  mem-max="1.2 MiB"/>
				<group name="Game"    include="^/Game/.*"    mem-max="2.2 GiB"/>
				<group name="Script"  include="^/Script/.*"  mem-max="120 MiB"/>
				<!-- A group for all other asset classes -->
				<group name="Other" include=".*" exclude="UntaggedAssetClass" mem-max="100 MiB"/>
			</grouping>
		</set>

		<platform name="Gen 5">
			<set name="System">
				<tracker name="Default">
					<tag name="Animation"                     mem-max="100 MiB"/>
					<tag name="Audio"                         mem-max="200 MiB"/>
					<tag name="Meshes"                        mem-max="300 MiB"/>
					<tag name="Textures"                      mem-max="3 GiB"/>
				</tracker>
			</set>
			<set name="Asset">
				<tracker name="Default">
					<tag name="UntaggedAsset"                 mem-max="2.1 GiB"/>
				</tracker>
				<tracker name="Platform">
					<tag name="UntaggedAsset"                 mem-max="3.1 GiB"/>
				</tracker>
				<grouping>
					<group name="Engine"                      mem-max="1.1 GiB"/>
					<group name="Game"                        mem-max="2.1 GiB"/>
				</grouping>
			</set>
			<set name="AssetClass">
				<tracker name="Default">
					<tag name="UntaggedAssetClass"            mem-max="1.1 GiB"/>
				</tracker>
				<tracker name="Platform">
					<tag name="UntaggedAssetClass"            mem-max="2.1 GiB"/>
				</tracker>
				<grouping>
					<group name="Engine"                      mem-max="1.1 GiB"/>
					<group name="Game"                        mem-max="2.1 GiB"/>
				</grouping>
			</set>
		</platform>

		<platform name="Mobile">
			<set name="System">
				<tracker name="Default">
					<tag name="Animation"                     mem-max="10 MiB"/>
					<tag name="Audio"                         mem-max="50 MiB"/>
					<tag name="Meshes"                        mem-max="100 MiB"/>
					<tag name="Textures"                      mem-max="1 GiB"/>
				</tracker>
			</set>
			<set name="Asset">
				<tracker name="Default">
					<tag name="UntaggedAsset"                 mem-max="0.8 GiB"/>
				</tracker>
				<tracker name="Platform">
					<tag name="UntaggedAsset"                 mem-max="1.0 GiB"/>
				</tracker>
				<grouping>
					<group name="Engine"                      mem-max="0.8 GiB"/>
					<group name="Game"                        mem-max="1.8 GiB"/>
				</grouping>
			</set>
			<set name="AssetClass">
				<tracker name="Default">
					<tag name="UntaggedAssetClass"            mem-max="0.7 GiB"/>
				</tracker>
				<tracker name="Platform">
					<tag name="UntaggedAssetClass"            mem-max="1.7 GiB"/>
				</tracker>
				<grouping>
					<group name="Engine"                      mem-max="0.7 MiB"/>
					<group name="Game"                        mem-max="1.7 GiB"/>
				</grouping>
			</set>
		</platform>
	</budget>

	<budget name="Empty">
	</budget>

</MemoryBudget>
