[SectionsToSave]
bCanSaveAllSections=true

[Insights.SessionBrowser]
; Whether Insights should signal to an UE application to auto-connect and start tracing when Insights is running.
AutoConnect=True
; Whether Insights should auto start analysis when a new live trace in received.
AutoStartAnalysis=False
; If specified, auto-start analysis will be enabled only for live trace sessions with this specified Platform.
AutoStartAnalysisPlatform=
; If specified, auto-start analysis will be enabled only for live trace sessions with this specified App Name.
AutoStartAnalysisAppName=