[SectionsToSave]
bCanSaveAllSections=True

[Insights.PersistentTimingView]
+Name=LoadingProfiler

[Insights]

; Search path used to locate a source file, if a file is not found using its full file path.
;SourceFilesSearchPath=D:\ue5-main\

; Add symbol search paths by adding to this array. Order is important.
;+SymbolSearchPaths=C:\Symbols
; Symbol server format is also accepted, such as the official Microsoft symbol servers (see https://docs.microsoft.com/en-us/windows-hardware/drivers/debugger/advanced-symsrv-use for details)
;+SymbolSearchPaths=srv*C:\Symbols;https://msdl.microsoft.com/download/symbols

[Insights.LoadingProfiler.TimingView]
; These options can be changed from the CPU/GPU drop down menus of Asset Loading Insights profiler / Timing view.

ShowAllGpuTracks=False
ShowGpuWorkTracks=True
ShowGpuWorkOverlays=True
ShowGpuWorkExtendedLines=True
ShowGpuFencesRelations=True
ShowGpuFencesTracks=False
ShowGpuFencesExtendedLines=True

ShowAllVerseTracks=False

ShowAllCpuTracks=False

ShowAllCpuSamplingTracks=False

ShowCpuCoreTracks=False
ShowContextSwitches=False
ShowContextSwitchOverlays=True
ShowContextSwitchExtendedLines=True
ShowNonTargetProcessEvents=True